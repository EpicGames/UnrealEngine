{
    "FileVersion" : 1,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Chaos Solver",
    "Description" : "Physics Solver",
    "Category": "Physics",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : true,
    "CanContainContent" : true,
    "IsBetaVersion" : true,
    "Installed" : false,
    "Modules": [
    {
      "Name": "ChaosSolverEditor",
      "Type": "Editor",
      "LoadingPhase": "Default"
    },
    ]
}
