// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosSolverEditor : ModuleRules
	{
        public ChaosSolverEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Slate",
                    "SlateCore",
                    "Engine",
					"EditorFramework",
                    "UnrealEd",
                    "PropertyEditor",
                    "RenderCore",
                    "RHI",
                    "ChaosSolverEngine",
                    "RawMesh",
                    "AssetTools",
                    "AssetRegistry",
					"ToolMenus",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"EditorStyle",
				}
				);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
