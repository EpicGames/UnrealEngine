// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ActorFactoryChaosSolver.h"
#include "Chaos/ChaosSolverActor.h"
#include "Chaos/ChaosSolver.h"
#include "AssetRegistry/AssetData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryChaosSolver)

#define LOCTEXT_NAMESPACE "ActorFactoryChaosSolver"

DEFINE_LOG_CATEGORY_STATIC(AFFS_Log, Log, All);

/*-----------------------------------------------------------------------------
UActorFactoryChaosSolver
-----------------------------------------------------------------------------*/
UActorFactoryChaosSolver::UActorFactoryChaosSolver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("ChaosSolverDisplayName", "ChaosSolver");
	NewActorClass = AChaosSolverActor::StaticClass();
}

bool UActorFactoryChaosSolver::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UChaosSolver::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoChaosSolverSpecified", "No ChaosSolver was specified.");
		return false;
	}

	return true;
}

void UActorFactoryChaosSolver::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UChaosSolver* ChaosSolver = CastChecked<UChaosSolver>(Asset);
	AChaosSolverActor* NewChaosSolverActor = CastChecked<AChaosSolverActor>(NewActor);
}

#undef LOCTEXT_NAMESPACE

