// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryChaosSolver.generated.h"

class AActor;
struct FAssetData;

UCLASS(MinimalAPI, config=Editor)
class UActorFactoryChaosSolver : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual void PostSpawnActor( UObject* Asset, AActor* NewActor ) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	//~ End UActorFactory Interface
};



