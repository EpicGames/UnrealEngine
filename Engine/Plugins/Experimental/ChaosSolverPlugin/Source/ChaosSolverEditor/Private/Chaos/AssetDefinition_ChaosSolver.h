// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "Chaos/ChaosSolver.h"
#include "AssetDefinition_ChaosSolver.generated.h"

UCLASS()
class UAssetDefinition_ChaosSolver : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override
	{
		return NSLOCTEXT("AssetDefinition", "AssetDefinition_ChaosSolver", "Chaos Solver");
	}

	virtual FLinearColor GetAssetColor() const override
	{
		return FColor(255, 192, 128);
	}

	virtual TSoftClassPtr<UObject> GetAssetClass() const override
	{
		return UChaosSolver::StaticClass();
	}

	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const TArray<FAssetCategoryPath> Categories = 
			{
				FAssetCategoryPath(EAssetCategoryPaths::Physics, NSLOCTEXT("AssetDefinition", "AssetDefinition_ChaosSolverSubMenu", "Chaos"), ECategoryMenuType::Section)
			};
		return Categories;
	}

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override { return EAssetCommandResult::Handled; }
	// UAssetDefinition End
};
