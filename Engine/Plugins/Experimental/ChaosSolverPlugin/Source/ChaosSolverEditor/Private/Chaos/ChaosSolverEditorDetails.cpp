// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSolverEditorDetails.h"

#include "IPropertyUtilities.h"
#include "DetailWidgetRow.h"
#include "EditorFontGlyphs.h"
#include "Chaos/ChaosSolverActor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IPropertyTypeCustomization> FChaosDebugSubstepControlCustomization::MakeInstance()
{
	return MakeShareable(new FChaosDebugSubstepControlCustomization);
}

void FChaosDebugSubstepControlCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const TSharedRef<IPropertyHandle> PropertyHandlePause = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FChaosDebugSubstepControl, bPause)).ToSharedRef();
	const TSharedRef<IPropertyHandle> PropertyHandleSubstep = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FChaosDebugSubstepControl, bSubstep)).ToSharedRef();
	const TSharedRef<IPropertyHandle> PropertyHandleStep = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FChaosDebugSubstepControl, bStep)).ToSharedRef();

	// Hide reset-to-default
	StructPropertyHandle->MarkResetToDefaultCustomized();

	// Find associated actor
	bool bHasBegunPlay = false;
	for (const TWeakObjectPtr<UObject>& SelectedObject: StructCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects())
	{
		if (AChaosSolverActor* const ChaosSolverActor = Cast<AChaosSolverActor>(SelectedObject.Get()))
		{
			// Refresh pause button callback
			const FSimpleDelegate OnPauseChanged = FSimpleDelegate::CreateSP(this, &FChaosDebugSubstepControlCustomization::RefreshPauseButton, PropertyHandlePause, SelectedObject);
			PropertyHandlePause->SetOnPropertyValueChanged(OnPauseChanged);

			// Refresh pause delegate for changes made through code to the actor
			// The struct property does not refresh when values are set through code, hence the need for this delegate
			ChaosSolverActor->ChaosDebugSubstepControl.OnPauseChanged = OnPauseChanged;

			// Check whether the actor has begun play
			bHasBegunPlay = ChaosSolverActor->HasActorBegunPlay();
			break;
		}
	}

	// Retrieve the current pause state
	bool bPaused = false;
	PropertyHandlePause->GetValue(bPaused);

	// Add buttons
	HeaderRow.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget(
				NSLOCTEXT("ChaosDebugSubstepControl", "ChaosDebugSubstepControl_Text", "Substep Control"),
				NSLOCTEXT("ChaosDebugSubstepControl", "ChaosDebugSubstepControl_ToolTip", "Allow to pause/substep/step the solver in its own debugging thread. Note: this is only available in multithreaded mode."))
		]
	.ValueContent()
		.MinDesiredWidth(125.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			.FillWidth(28.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(4.0f)
				.ToolTipText(NSLOCTEXT("ChaosSolverActor", "Pause_ToolTip", "Pause/Resume"))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FChaosDebugSubstepControlCustomization::OnPause, PropertyHandlePause)
				[
					SAssignNew(TextBlockPause, STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(bPaused ? FEditorFontGlyphs::Play: FEditorFontGlyphs::Pause)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			.FillWidth(28.0f)
			[
				SAssignNew(ButtonSubstep, SButton)
				.IsEnabled_Lambda([bPaused, bHasBegunPlay]() { return bPaused && bHasBegunPlay; })
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(4.0f)
				.ToolTipText(NSLOCTEXT("ChaosSolverActor", "Substep_ToolTip", "Substep"))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FChaosDebugSubstepControlCustomization::OnSubstep, PropertyHandleSubstep)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Step_Forward)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			.FillWidth(28.0f)
			[
				SAssignNew(ButtonStep, SButton)
				.IsEnabled_Lambda([bPaused, bHasBegunPlay]() { return bPaused && bHasBegunPlay; })
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ForegroundColor(FSlateColor::UseForeground())
				.ContentPadding(4.0f)
				.ToolTipText(NSLOCTEXT("ChaosSolverActor", "Step_ToolTip", "Step"))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.OnClicked(this, &FChaosDebugSubstepControlCustomization::OnStep, PropertyHandleStep)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Fast_Forward)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}

void FChaosDebugSubstepControlCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, class IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*StructCustomizationUtils*/)
{
}

FReply FChaosDebugSubstepControlCustomization::OnPause(TSharedRef<IPropertyHandle> PropertyHandle)
{
	bool bPaused = false;
	PropertyHandle->GetValue(bPaused);
	bPaused = !bPaused;
	PropertyHandle->SetValue(bPaused);

	return FReply::Handled();
}

FReply FChaosDebugSubstepControlCustomization::OnSubstep(TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(true);
	return FReply::Handled();
}

FReply FChaosDebugSubstepControlCustomization::OnStep(TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue(true);
	return FReply::Handled();
}

void FChaosDebugSubstepControlCustomization::RefreshPauseButton(TSharedRef<IPropertyHandle> PropertyHandle, TWeakObjectPtr<UObject> Object)
{
	const AChaosSolverActor* const ChaosSolverActor = Cast<AChaosSolverActor>(Object.Get());
	const bool bHasBegunPlay = ChaosSolverActor && ChaosSolverActor->HasActorBegunPlay();

	bool bPaused = false;
	PropertyHandle->GetValue(bPaused);

	if (TextBlockPause.IsValid())
	{
		TextBlockPause->SetText(bPaused ? FEditorFontGlyphs::Play: FEditorFontGlyphs::Pause);
	}
	if (ButtonSubstep.IsValid())
	{
		ButtonSubstep->SetEnabled(bPaused && bHasBegunPlay);
	}
	if (ButtonStep.IsValid())
	{
		ButtonStep->SetEnabled(bPaused && bHasBegunPlay);
	}
}
