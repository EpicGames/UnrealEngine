// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPropertyTypeCustomization.h"

class FReply;
class SButton;
class STextBlock;

class IDetailLayoutBuilder;

/** Details customization for Chaos debug substep controls. */
class FChaosDebugSubstepControlCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	FReply OnPause(TSharedRef<IPropertyHandle> PropertyHandle);
	FReply OnSubstep(TSharedRef<IPropertyHandle> PropertyHandle);
	FReply OnStep(TSharedRef<IPropertyHandle> PropertyHandle);

	void RefreshPauseButton(TSharedRef<IPropertyHandle> PropertyHandle, TWeakObjectPtr<UObject> Object);

private:
	TSharedPtr<STextBlock> TextBlockPause;
	TSharedPtr<SButton> ButtonStep;
	TSharedPtr<SButton> ButtonSubstep;
};
