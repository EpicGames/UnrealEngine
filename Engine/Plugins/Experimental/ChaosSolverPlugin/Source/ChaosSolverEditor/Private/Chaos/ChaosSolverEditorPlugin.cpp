// Copyright Epic Games, Inc. All Rights Reserved.


#include "Chaos/ChaosSolverEditorPlugin.h"

#include "Chaos/ChaosSolverEditorStyle.h"
#include "PropertyEditorModule.h"
#include "ChaosSolverEditorDetails.h"

TOptional<FChaosSolverEditorStyle> FChaosSolverEditorStyle::Singleton;

IMPLEMENT_MODULE( IChaosSolverEditorPlugin, ChaosSolverEditor )

void IChaosSolverEditorPlugin::StartupModule()
{
	FChaosSolverEditorStyle::Get();

	if (GIsEditor && !IsRunningCommandlet())
	{
	}

	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("ChaosDebugSubstepControl", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosDebugSubstepControlCustomization::MakeInstance));
}


void IChaosSolverEditorPlugin::ShutdownModule()
{
	// Unregister details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout("ChaosDebugSubstepControl");
}



