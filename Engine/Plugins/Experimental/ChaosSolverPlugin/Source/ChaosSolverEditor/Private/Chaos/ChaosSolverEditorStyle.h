// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

class FChaosSolverEditorStyle final : public FSlateStyleSet
{
public:
	FChaosSolverEditorStyle() : FSlateStyleSet("ChaosSolverEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

#if !IS_MONOLITHIC
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/ChaosSolverPlugin/Resources"));
#endif

		Set("ClassIcon.ChaosSolver", new FSlateVectorImageBrush(RootToContentDir(TEXT("ChaosSolver_16.svg")), Icon16x16));
		Set("ClassThumbnail.ChaosSolver", new FSlateVectorImageBrush(RootToContentDir(TEXT("ChaosSolver_64.svg")), Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FChaosSolverEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FChaosSolverEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FChaosSolverEditorStyle> Singleton;
};

