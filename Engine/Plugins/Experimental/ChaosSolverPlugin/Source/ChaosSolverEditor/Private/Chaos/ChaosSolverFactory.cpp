// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosSolverFactory.h"

#include "Chaos/ChaosSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSolverFactory)

#define LOCTEXT_NAMESPACE "ChaosSolver"

/////////////////////////////////////////////////////
// ChaosSolverFactory

UChaosSolverFactory::UChaosSolverFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UChaosSolver::StaticClass();
}

UChaosSolver* UChaosSolverFactory::StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return static_cast<UChaosSolver*>(NewObject<UChaosSolver>(InParent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone));
}

UObject* UChaosSolverFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UChaosSolver* NewChaosSolver = StaticFactoryCreateNew(Class, InParent, Name, Flags, Context, Warn);
	NewChaosSolver->MarkPackageDirty();
	return NewChaosSolver;
}

#undef LOCTEXT_NAMESPACE




