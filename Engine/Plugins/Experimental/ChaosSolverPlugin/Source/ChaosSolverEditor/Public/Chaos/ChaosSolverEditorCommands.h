// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"

class UStaticMesh;
class USkeletalMesh;

/**
* The public interface to this module
*/
class FChaosSolverCommands
{
};
