// Copyright Epic Games, Inc. All Rights Reserved.

/** Factory which allows import of an ChaosSolverAsset */

#pragma once

#include "Factories/Factory.h"

#include "ChaosSolverFactory.generated.h"

#define UE_API CHAOSSOLVEREDITOR_API

class UChaosSolver;


/**
* Factory for Simple Cube
*/

UCLASS(MinimalAPI)
class UChaosSolverFactory : public UFactory
{
    GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	static UE_API UChaosSolver* StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);
};

#undef UE_API
